// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"
#include "Modules/ModuleManager.h"

// Implement the modules
IMPLEMENT_MODULE(FDefaultModuleImpl, UnrealGame);

// Variables referenced from Core, and usually declared by IMPLEMENT_PRIMARY_GAME_MODULE. Since UnrealGame is game-agnostic, implement them here.
#if IS_MONOLITHIC
PER_MODULE_BOILERPLATE
bool GIsGameAgnosticExe = true;
TCHAR GInternalProjectName[64] = TEXT("");
IMPLEMENT_FOREIGN_ENGINE_DIR()
#endif
