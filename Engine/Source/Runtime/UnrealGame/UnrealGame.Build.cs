// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealGame : ModuleRules
{
	public UnrealGame(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");


        //DynamicallyLoadedModuleNames.Add("OnlineSubsystemNull");

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "Core", "CoreUObject", "Engine" });//, "OnlineSubsystem", "OnlineSubsystemUtils" });
            //DynamicallyLoadedModuleNames.Add("OnlineSubsystemIOS");
			if (Target.Platform == UnrealTargetPlatform.IOS)
			{
                //DynamicallyLoadedModuleNames.Add("OnlineSubsystemFacebook");
				DynamicallyLoadedModuleNames.Add("IOSAdvertising");
			}
		}
		else if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "Core", "CoreUObject", "Engine" });//, "OnlineSubsystem", "OnlineSubsystemUtils" });
			DynamicallyLoadedModuleNames.Add("AndroidAdvertising");
            //DynamicallyLoadedModuleNames.Add("OnlineSubsystemGooglePlay");
		}
	}
}
